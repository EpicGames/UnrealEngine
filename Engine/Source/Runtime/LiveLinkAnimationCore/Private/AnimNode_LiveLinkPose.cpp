// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_LiveLinkPose.h"

#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimTrace.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "LiveLinkCustomVersion.h"
#include "LiveLinkRemapAsset.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_LiveLinkPose)

FAnimNode_LiveLinkPose::FAnimNode_LiveLinkPose() 
	: RetargetAsset(ULiveLinkRemapAsset::StaticClass())
	, CurrentRetargetAsset(nullptr)
	, LiveLinkClient_AnyThread(nullptr)
	, CachedDeltaTime(0.0f)
{
}

void FAnimNode_LiveLinkPose::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	CurrentRetargetAsset = nullptr;

	Super::OnInitializeAnimInstance(InProxy, InAnimInstance);
}

void FAnimNode_LiveLinkPose::BuildPoseFromAnimData(const FLiveLinkSubjectFrameData& LiveLinkData, FPoseContext& Output)
{
	const FLiveLinkSkeletonStaticData* SkeletonData = LiveLinkData.StaticData.Cast<FLiveLinkSkeletonStaticData>();
	const FLiveLinkAnimationFrameData* FrameData = LiveLinkData.FrameData.Cast<FLiveLinkAnimationFrameData>();
	check(SkeletonData);
	check(FrameData);

	if (FrameData->Transforms.Num() || FrameData->PropertyValues.Num())
	{
		CurrentRetargetAsset->BuildPoseFromAnimationData(CachedDeltaTime, SkeletonData, FrameData, Output.Pose);
		CurrentRetargetAsset->BuildPoseAndCurveFromBaseData(CachedDeltaTime, SkeletonData, FrameData, Output.Pose, Output.Curve);
	}

	CachedDeltaTime = 0.f; // Reset so that if we evaluate again we don't "create" time inside of the retargeter
}

void FAnimNode_LiveLinkPose::BuildPoseFromCurveData(const FLiveLinkSubjectFrameData& LiveLinkData, FPoseContext& Output)
{
	const FLiveLinkBaseStaticData* BaseStaticData = LiveLinkData.StaticData.Cast<FLiveLinkBaseStaticData>();
	const FLiveLinkBaseFrameData* BaseFrameData = LiveLinkData.FrameData.Cast<FLiveLinkBaseFrameData>();
	check(BaseStaticData);
	check(BaseFrameData);

	if (BaseFrameData->PropertyValues.Num())
	{
		CurrentRetargetAsset->BuildPoseAndCurveFromBaseData(CachedDeltaTime, BaseStaticData, BaseFrameData, Output.Pose, Output.Curve);
	}
	CachedDeltaTime = 0.f; // Reset so that if we evaluate again we don't "create" time inside of the retargeter
}

void FAnimNode_LiveLinkPose::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	InputPose.Initialize(Context);

	if (CachedLiveLinkData.IsValid() == false)
	{
		CachedLiveLinkData = MakeShared<FLiveLinkSubjectFrameData>();
	}
}

void FAnimNode_LiveLinkPose::PreUpdate(const UAnimInstance* InAnimInstance)
{
	ILiveLinkClient* ThisFrameClient = nullptr;
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ThisFrameClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	}
	LiveLinkClient_AnyThread = ThisFrameClient;

	// Protection as a class graph pin does not honor rules on abstract classes and NoClear
	UClass* RetargetAssetPtr = RetargetAsset.Get();
	if (!RetargetAssetPtr || RetargetAssetPtr->HasAnyClassFlags(CLASS_Abstract))
	{
		RetargetAssetPtr = ULiveLinkRemapAsset::StaticClass();
		RetargetAsset = RetargetAssetPtr;
	}

	if (!CurrentRetargetAsset || RetargetAssetPtr != CurrentRetargetAsset->GetClass())
	{
		CurrentRetargetAsset = NewObject<ULiveLinkRetargetAsset>(const_cast<UAnimInstance*>(InAnimInstance), RetargetAssetPtr);
		CurrentRetargetAsset->Initialize();
	}
}

void FAnimNode_LiveLinkPose::Update_AnyThread(const FAnimationUpdateContext & Context)
{
	InputPose.Update(Context);

	GetEvaluateGraphExposedInputs().Execute(Context);

	// Accumulate Delta time from update
	CachedDeltaTime += Context.GetDeltaTime();

	TRACE_ANIM_NODE_VALUE(Context, TEXT("SubjectName"), LiveLinkSubjectName.Name);
}

void FAnimNode_LiveLinkPose::Evaluate_AnyThread(FPoseContext& Output)
{
	InputPose.Evaluate(Output);

	if (!LiveLinkClient_AnyThread || !CurrentRetargetAsset)
	{
		return;
	}

	FLiveLinkSubjectFrameData SubjectFrameData;

	if (bDoLiveLinkEvaluation)
	{
		// Invalidate cached evaluated Role to make sure we have a valid one during the last evaluation when using it
		CachedEvaluatedRole = nullptr;

		TSubclassOf<ULiveLinkRole> SubjectRole = LiveLinkClient_AnyThread->GetSubjectRole_AnyThread(LiveLinkSubjectName);
		if (SubjectRole)
		{
			if (LiveLinkClient_AnyThread->DoesSubjectSupportsRole_AnyThread(LiveLinkSubjectName, ULiveLinkAnimationRole::StaticClass()))
			{
				//Process animation data if the subject is from that type
				if (LiveLinkClient_AnyThread->EvaluateFrame_AnyThread(LiveLinkSubjectName, ULiveLinkAnimationRole::StaticClass(), SubjectFrameData))
				{
					BuildPoseFromAnimData(SubjectFrameData, Output);

					CachedEvaluatedRole = ULiveLinkAnimationRole::StaticClass();
				}
			}
			else if (LiveLinkClient_AnyThread->DoesSubjectSupportsRole_AnyThread(LiveLinkSubjectName, ULiveLinkBasicRole::StaticClass()))
			{
				//Otherwise, fetch basic data that contains property / curve data
				if (LiveLinkClient_AnyThread->EvaluateFrame_AnyThread(LiveLinkSubjectName, ULiveLinkBasicRole::StaticClass(), SubjectFrameData))
				{
					BuildPoseFromCurveData(SubjectFrameData, Output);

					CachedEvaluatedRole = ULiveLinkBasicRole::StaticClass();
				}
			}

			// Keep a cached version of evaluated data to continue building pose with it when we pause evaluation
			CachedLiveLinkData->StaticData = MoveTemp(SubjectFrameData.StaticData);
			CachedLiveLinkData->FrameData = MoveTemp(SubjectFrameData.FrameData);
		}
	}
	else
	{
		if(CachedLiveLinkData && CachedEvaluatedRole)
		{
			if (CachedEvaluatedRole == ULiveLinkAnimationRole::StaticClass())
			{
				BuildPoseFromAnimData(*CachedLiveLinkData, Output);
			}
			else if (CachedEvaluatedRole == ULiveLinkBasicRole::StaticClass())
			{
				BuildPoseFromCurveData(*CachedLiveLinkData, Output);
			}
		}
	}
}

void FAnimNode_LiveLinkPose::CacheBones_AnyThread(const FAnimationCacheBonesContext & Context)
{
	Super::CacheBones_AnyThread(Context);
	InputPose.CacheBones(Context);
}

void FAnimNode_LiveLinkPose::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = FString::Printf(TEXT("LiveLink - SubjectName: %s"), *LiveLinkSubjectName.ToString());

	DebugData.AddDebugItem(DebugLine);
	InputPose.GatherDebugData(DebugData);
}

bool FAnimNode_LiveLinkPose::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FLiveLinkCustomVersion::GUID);
	
	UScriptStruct* Struct = FAnimNode_LiveLinkPose::StaticStruct();
	
	if (Ar.IsLoading() || Ar.IsSaving())
	{
		Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);
	}

#if WITH_EDITORONLY_DATA
	//Take old data and put it in new data structure
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FLiveLinkCustomVersion::GUID) < FLiveLinkCustomVersion::NewLiveLinkRoleSystem)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			LiveLinkSubjectName.Name = SubjectName_DEPRECATED;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif

	return true;
}


