// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkInstance)

void FLiveLinkInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	FAnimInstanceProxy::Initialize(InAnimInstance);

	// initialize node manually 
	FAnimationInitializeContext InitContext(this);
	PoseNode.Initialize_AnyThread(InitContext);
}

void FLiveLinkInstanceProxy::PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds)
{
	Super::PreUpdate(InAnimInstance, DeltaSeconds);

	if (PoseNode.HasPreUpdate())
	{
		PoseNode.PreUpdate(InAnimInstance);
	}
}

bool FLiveLinkInstanceProxy::Evaluate(FPoseContext& Output)
{
	PoseNode.Evaluate_AnyThread(Output);

	return true;
}

void FLiveLinkInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	UpdateCounter.Increment();

	PoseNode.Update_AnyThread(InContext);
	
	if(ULiveLinkInstance* Instance = Cast<ULiveLinkInstance>(GetAnimInstanceObject()))
	{
		Instance->CurrentRetargetAsset = PoseNode.CurrentRetargetAsset; //Cache for GC
	}
}

FAnimInstanceProxy* ULiveLinkInstance::CreateAnimInstanceProxy()
{
	return new FLiveLinkInstanceProxy(this);
}

void ULiveLinkInstance::DestroyAnimInstanceProxy(FAnimInstanceProxy* InProxy)
{
	Super::DestroyAnimInstanceProxy(InProxy);
	CurrentRetargetAsset = nullptr;
}
