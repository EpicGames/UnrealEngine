// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkRemapAsset.h"

#include "BonePose.h"
#include "Engine/Blueprint.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "LiveLinkTypes.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkRemapAsset)

  
ULiveLinkRemapAsset::ULiveLinkRemapAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	UBlueprint* Blueprint = Cast<UBlueprint>(GetClass()->ClassGeneratedBy);
	if (Blueprint)
	{
		OnBlueprintCompiledDelegate = Blueprint->OnCompiled().AddUObject(this, &ULiveLinkRemapAsset::OnBlueprintClassCompiled);
	}
#endif
}

void ULiveLinkRemapAsset::BeginDestroy()
{
#if WITH_EDITOR
	if (OnBlueprintCompiledDelegate.IsValid())
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(GetClass()->ClassGeneratedBy))
		{
			Blueprint->OnCompiled().Remove(OnBlueprintCompiledDelegate);
		}
		OnBlueprintCompiledDelegate.Reset();
	}
#endif

	Super::BeginDestroy();
}

void ULiveLinkRemapAsset::OnBlueprintClassCompiled(UBlueprint* TargetBlueprint)
{
	BoneNameMap.Reset();
	CurveNameMap.Reset();
}

void MakeCurveMapFromFrame(const FCompactPose& InPose, const FLiveLinkBaseStaticData* InBaseStaticData, const FLiveLinkBaseFrameData* InFrameData, const TArray<FName, TMemStackAllocator<>>& TransformedCurveNames, TMap<FName, float>& OutCurveMap)
{
	OutCurveMap.Reset();
	OutCurveMap.Reserve(InFrameData->PropertyValues.Num());

	if (InBaseStaticData->PropertyNames.Num() == InFrameData->PropertyValues.Num())
	{
		for (int32 CurveIdx = 0; CurveIdx < InBaseStaticData->PropertyNames.Num(); ++CurveIdx)
		{
			const float PropertyValue = InFrameData->PropertyValues[CurveIdx];
			if (FMath::IsFinite(PropertyValue))
			{
				OutCurveMap.Add(TransformedCurveNames[CurveIdx]) = PropertyValue;
			}
		}
	}
}

void ULiveLinkRemapAsset::BuildPoseFromAnimationData(float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose)
{
	const TArray<FName>& SourceBoneNames = InSkeletonData->BoneNames;

	TArray<FName, TMemStackAllocator<>> TransformedBoneNames;
	TransformedBoneNames.Reserve(SourceBoneNames.Num());

	for (const FName& SrcBoneName : SourceBoneNames)
	{
		FName* TargetBoneName = BoneNameMap.Find(SrcBoneName);
		if (TargetBoneName == nullptr)
		{
			FName NewName = GetRemappedBoneName(SrcBoneName);
			TransformedBoneNames.Add(NewName);
			BoneNameMap.Add(SrcBoneName, NewName);
		}
		else
		{
			TransformedBoneNames.Add(*TargetBoneName);
		}
	}

	for (int32 i = 0; i < TransformedBoneNames.Num(); ++i)
	{
		FName BoneName = TransformedBoneNames[i];

		FTransform BoneTransform = InFrameData->Transforms[i];

		int32 MeshIndex = OutPose.GetBoneContainer().GetPoseBoneIndexForBoneName(BoneName);
		if (MeshIndex != INDEX_NONE)
		{
			FCompactPoseBoneIndex CPIndex = OutPose.GetBoneContainer().MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshIndex));
			if (CPIndex != INDEX_NONE)
			{
				OutPose[CPIndex] = BoneTransform;
			}
		}
	}
}

void ULiveLinkRemapAsset::BuildPoseAndCurveFromBaseData(float DeltaTime, const FLiveLinkBaseStaticData* InBaseStaticData, const FLiveLinkBaseFrameData* InBaseFrameData, FCompactPose& OutPose, FBlendedCurve& OutCurve)
{
	const TArray<FName>& SourceCurveNames = InBaseStaticData->PropertyNames;
	TArray<FName, TMemStackAllocator<>> TransformedCurveNames;
	TransformedCurveNames.Reserve(SourceCurveNames.Num());

	for (const FName& SrcCurveName : SourceCurveNames)
	{
		FName* TargetCurveName = CurveNameMap.Find(SrcCurveName);
		if (TargetCurveName == nullptr)
		{
			FName NewName = GetRemappedCurveName(SrcCurveName);
			TransformedCurveNames.Add(NewName);
			CurveNameMap.Add(SrcCurveName, NewName);
		}
		else
		{
			TransformedCurveNames.Add(*TargetCurveName);
		}
	}

	TMap<FName, float> BPCurveValues;

	MakeCurveMapFromFrame(OutPose, InBaseStaticData, InBaseFrameData, TransformedCurveNames, BPCurveValues);

	RemapCurveElements(BPCurveValues);

	BuildCurveData(BPCurveValues, OutPose, OutCurve);
}

FName ULiveLinkRemapAsset::GetRemappedBoneName_Implementation(FName BoneName) const
{
	return BoneName;
}

FName ULiveLinkRemapAsset::GetRemappedCurveName_Implementation(FName CurveName) const
{
	return CurveName;
}

void ULiveLinkRemapAsset::RemapCurveElements_Implementation(TMap<FName, float>& CurveItems) const
{
}
