// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkRetargetAsset.h"

#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimTypes.h"
#include "Animation/Skeleton.h"
#include "BonePose.h"
#include "Animation/AnimCurveUtils.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "Roles/LiveLinkAnimationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LiveLinkRetargetAsset)

ULiveLinkRetargetAsset::ULiveLinkRetargetAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void ULiveLinkRetargetAsset::ApplyCurveValue(const USkeleton* Skeleton, const FName CurveName, const float CurveValue, FBlendedCurve& OutCurve) const
{
	OutCurve.Set(CurveName, CurveValue);
}

void ULiveLinkRetargetAsset::BuildCurveData(const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, const FCompactPose& InPose, FBlendedCurve& OutCurve) const
{
	const USkeleton* Skeleton = InPose.GetBoneContainer().GetSkeletonAsset();

	if (InSkeletonData->PropertyNames.Num() == InFrameData->PropertyValues.Num())
	{
		auto GetCurveNameFromIndex = [InSkeletonData](int32 InCurveIndex)
		{
			return InSkeletonData->PropertyNames[InCurveIndex];
		};

		auto GetCurveValueFromIndex = [InFrameData](int32 InCurveIndex)
		{
			const float Curve = InFrameData->PropertyValues[InCurveIndex];
			if (FMath::IsFinite(Curve))
			{
				return InFrameData->PropertyValues[InCurveIndex];
			}
			else
			{
				return 0.0f;
			}
		};

		FBlendedCurve Curve;
		UE::Anim::FCurveUtils::BuildUnsorted(Curve, InSkeletonData->PropertyNames.Num(), GetCurveNameFromIndex, GetCurveValueFromIndex);
		OutCurve.Combine(Curve);
	}
}

void ULiveLinkRetargetAsset::BuildCurveData(const TMap<FName, float>& CurveMap, const FCompactPose& InPose, FBlendedCurve& OutCurve) const
{
	FBlendedCurve Curve;
	UE::Anim::FCurveUtils::BuildUnsorted(Curve, CurveMap);
	OutCurve.Combine(Curve);
}
