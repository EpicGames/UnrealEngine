// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNodeBase.h"

#include "CoreMinimal.h"
#include "LiveLinkRetargetAsset.h"
#include "LiveLinkTypes.h"
#include "Templates/SubclassOf.h"

#include "AnimNode_LiveLinkPose.generated.h"


class ILiveLinkClient;
class ULiveLinkRole;
struct FLiveLinkSubjectFrameData;


PRAGMA_DISABLE_DEPRECATION_WARNINGS

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_LiveLinkPose : public FAnimNode_Base
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Input)
	FPoseLink InputPose;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = SourceData, meta = (PinShownByDefault))
	FLiveLinkSubjectName LiveLinkSubjectName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Transient, Category = SourceData, meta = (PinShownByDefault))
	bool bDoLiveLinkEvaluation = true;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(4.23, "FName SubjectName is deprecated. Use the SubjectName of type FLiveLinkSubjectName instead.")
	UPROPERTY()
	FName SubjectName_DEPRECATED;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, NoClear, Category = Retarget, meta = (NeverAsPin))
	TSubclassOf<ULiveLinkRetargetAsset> RetargetAsset;

	UPROPERTY(transient)
	TObjectPtr<ULiveLinkRetargetAsset> CurrentRetargetAsset;

public:
	LIVELINKANIMATIONCORE_API FAnimNode_LiveLinkPose();

	//~ FAnimNode_Base interface
	LIVELINKANIMATIONCORE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	LIVELINKANIMATIONCORE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext & Context) override;
	LIVELINKANIMATIONCORE_API virtual void Update_AnyThread(const FAnimationUpdateContext & Context) override;
	LIVELINKANIMATIONCORE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	virtual bool HasPreUpdate() const { return true; }
	LIVELINKANIMATIONCORE_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	LIVELINKANIMATIONCORE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	//~ End of FAnimNode_Base interface

	LIVELINKANIMATIONCORE_API bool Serialize(FArchive& Ar);

protected:
	LIVELINKANIMATIONCORE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	
	/** Build output pose using evaluated LiveLink animation data */
	LIVELINKANIMATIONCORE_API void BuildPoseFromAnimData(const FLiveLinkSubjectFrameData& LiveLinkData, FPoseContext& Output);
	
	/** Build output pose using evaluated LiveLink curve (Basic Role) data */
	LIVELINKANIMATIONCORE_API void BuildPoseFromCurveData(const FLiveLinkSubjectFrameData& LiveLinkData, FPoseContext& Output);


private:

	ILiveLinkClient* LiveLinkClient_AnyThread;

	// Delta time from update so that it can be passed to retargeter
	float CachedDeltaTime;

	/** Cached LiveLink evaluated data from last frame */
	TSharedPtr<FLiveLinkSubjectFrameData> CachedLiveLinkData;

	/** Cached LiveLink Role evaluated last frame */
	TSubclassOf<ULiveLinkRole> CachedEvaluatedRole;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS


template<> struct TStructOpsTypeTraits<FAnimNode_LiveLinkPose> : public TStructOpsTypeTraitsBase2<FAnimNode_LiveLinkPose>
{
	enum 
	{ 
		WithSerializer = true
	};
};
