// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "AnimNode_LiveLinkPose.h"

#include "LiveLinkInstance.generated.h"

class ULiveLinkRetargetAsset;

/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
struct FLiveLinkInstanceProxy : public FAnimInstanceProxy
{
public:
	friend struct FAnimNode_LiveLinkPose;

	GENERATED_BODY()

	FLiveLinkInstanceProxy()
	{
	}

	FLiveLinkInstanceProxy(UAnimInstance* InAnimInstance)
		: FAnimInstanceProxy(InAnimInstance)
	{
	}

	LIVELINKANIMATIONCORE_API virtual void Initialize(UAnimInstance* InAnimInstance) override;
	LIVELINKANIMATIONCORE_API virtual void PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds) override;
	LIVELINKANIMATIONCORE_API virtual bool Evaluate(FPoseContext& Output) override;
	LIVELINKANIMATIONCORE_API virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_LiveLinkPose PoseNode;
};

UCLASS(transient, NotBlueprintable, MinimalAPI)
class ULiveLinkInstance : public UAnimInstance
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category="Animation|Live Link")
	void SetSubject(FLiveLinkSubjectName SubjectName)
	{
		GetProxyOnGameThread<FLiveLinkInstanceProxy>().PoseNode.LiveLinkSubjectName = SubjectName;
	}

	UFUNCTION(BlueprintCallable, Category = "Animation|Live Link")
	void SetRetargetAsset(TSubclassOf<ULiveLinkRetargetAsset> RetargetAsset)
	{
		GetProxyOnGameThread<FLiveLinkInstanceProxy>().PoseNode.RetargetAsset = RetargetAsset;
	}

	UFUNCTION(BlueprintCallable, Category = "Animation|Live Link")
	void EnableLiveLinkEvaluation(bool bDoEnable)
	{
		GetProxyOnGameThread<FLiveLinkInstanceProxy>().PoseNode.bDoLiveLinkEvaluation = bDoEnable;
	}

	bool GetEnableLiveLinkEvaluation()
	{
		return GetProxyOnGameThread<FLiveLinkInstanceProxy>().PoseNode.bDoLiveLinkEvaluation;
	}

protected:
	LIVELINKANIMATIONCORE_API virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
	LIVELINKANIMATIONCORE_API virtual void DestroyAnimInstanceProxy(FAnimInstanceProxy* InProxy) override;

	// Cache for GC
	UPROPERTY(transient)
	TObjectPtr<ULiveLinkRetargetAsset> CurrentRetargetAsset;

	friend FLiveLinkInstanceProxy;
};
