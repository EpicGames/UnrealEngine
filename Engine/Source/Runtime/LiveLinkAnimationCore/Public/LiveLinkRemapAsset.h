// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "LiveLinkRetargetAsset.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkRemapAsset.generated.h"

class UBlueprint;
class UObject;
struct FBlendedCurve;
struct FCompactPose;
struct FFrame;
struct FLiveLinkAnimationFrameData;
struct FLiveLinkBaseFrameData;
struct FLiveLinkBaseStaticData;
struct FLiveLinkSkeletonStaticData;

// Remap asset for data coming from Live Link. Allows simple application of bone transforms into current pose based on name remapping only
UCLASS(Blueprintable, MinimalAPI)
class ULiveLinkRemapAsset : public ULiveLinkRetargetAsset
{
	GENERATED_UCLASS_BODY()

	virtual ~ULiveLinkRemapAsset() = default;

	//~ Begin UObject Interface
	LIVELINKANIMATIONCORE_API virtual void BeginDestroy() override;
	//~ End UObject Interface

	//~ Begin ULiveLinkRetargetAsset interface
	LIVELINKANIMATIONCORE_API virtual void BuildPoseFromAnimationData(float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose) override;
	LIVELINKANIMATIONCORE_API virtual void BuildPoseAndCurveFromBaseData(float DeltaTime, const FLiveLinkBaseStaticData* InBaseStaticData, const FLiveLinkBaseFrameData* InBaseFrameData, FCompactPose& OutPose, FBlendedCurve& OutCurve) override;
	//~ End ULiveLinkRetargetAsset interface

	/** Blueprint Implementable function for getting a remapped bone name from the original */
	UFUNCTION(BlueprintNativeEvent, Category = "Live Link Remap")
	LIVELINKANIMATIONCORE_API FName GetRemappedBoneName(FName BoneName) const;

	/** Blueprint Implementable function for getting a remapped curve name from the original */
	UFUNCTION(BlueprintNativeEvent, Category = "Live Link Remap")
	LIVELINKANIMATIONCORE_API FName GetRemappedCurveName(FName CurveName) const;

	/** Blueprint Implementable function for remapping, adding or otherwise modifying the curve element data from Live Link. This is run after GetRemappedCurveName */
	UFUNCTION(BlueprintNativeEvent, Category = "Live Link Remap")
	LIVELINKANIMATIONCORE_API void RemapCurveElements(UPARAM(ref)TMap<FName, float>& CurveItems) const;

private:

	LIVELINKANIMATIONCORE_API void OnBlueprintClassCompiled(UBlueprint* TargetBlueprint);

	// Name mapping between source bone name and transformed bone name
	// (returned from GetRemappedBoneName)
	TMap<FName, FName> BoneNameMap;

	// Name mapping between source curve name and transformed curve name
	// (returned from GetRemappedCurveName)
	TMap<FName, FName> CurveNameMap;

#if WITH_EDITOR
	/** Blueprint.OnCompiled delegate handle */
	FDelegateHandle OnBlueprintCompiledDelegate;
#endif
};
