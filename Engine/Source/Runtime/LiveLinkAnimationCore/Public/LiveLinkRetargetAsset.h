// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LiveLinkRetargetAsset.generated.h"

class USkeleton;
struct FBlendedCurve;
struct FCompactPose;
struct FLiveLinkAnimationFrameData;
struct FLiveLinkBaseFrameData;
struct FLiveLinkBaseStaticData;
struct FLiveLinkSkeletonStaticData;

// Base class for retargeting live link data. 
UCLASS(Abstract, MinimalAPI)
class ULiveLinkRetargetAsset : public UObject
{
	GENERATED_UCLASS_BODY()

	// Takes the supplied curve name and value and applies it to the blended curve (as approriate given the supplied skeleton
	LIVELINKANIMATIONCORE_API void ApplyCurveValue(const USkeleton* Skeleton, const FName CurveName, const float CurveValue, FBlendedCurve& OutCurve) const;

	// Builds curve data into OutCurve from the supplied live link frame
	LIVELINKANIMATIONCORE_API void BuildCurveData(const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, const FCompactPose& InPose, FBlendedCurve& OutCurve) const;

	// Builds curve data into OutCurve from the supplied map of curve name to float
	LIVELINKANIMATIONCORE_API void BuildCurveData(const TMap<FName, float>& CurveMap, const FCompactPose& InPose, FBlendedCurve& OutCurve) const;

	// Called once when the retargeter is created 
	virtual void Initialize() {}

	// Build OutPose from AnimationData if subject was from this type
	virtual void BuildPoseFromAnimationData(float DeltaTime, const FLiveLinkSkeletonStaticData* InSkeletonData, const FLiveLinkAnimationFrameData* InFrameData, FCompactPose& OutPose) {}

	// Build OutPose and OutCurve from the basic data. Called for every type of subjects
	virtual void BuildPoseAndCurveFromBaseData(float DeltaTime, const FLiveLinkBaseStaticData* InBaseStaticData, const FLiveLinkBaseFrameData* InBaseFrameData, FCompactPose& OutPose, FBlendedCurve& OutCurve) {}
};
